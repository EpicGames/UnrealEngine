// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataEditor.h"
#include "DerivedDataInformation.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "SDerivedDataDialogs.h"
#include "SDerivedDataStatusBar.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "DerivedDataEditor"

IMPLEMENT_MODULE(FDerivedDataEditor, DerivedDataEditor );

static const FName DerivedDataResourceUsageTabName = FName(TEXT("DerivedDataResourceUsage"));
static const FName DerivedDataCacheStatisticsTabName = FName(TEXT("DerivedDataCacheStatistics"));

void FDerivedDataEditor::StartupModule()
{
	const FSlateIcon ResourceUsageIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.ResourceUsage");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DerivedDataResourceUsageTabName, FOnSpawnTab::CreateRaw(this, &FDerivedDataEditor::CreateResourceUsageTab))
		.SetDisplayName(LOCTEXT("DerivedDataResourceUsageTabTitle", "Resource Usage"))
		.SetTooltipText(LOCTEXT("DerivedDataResourceUsageTabToolTipText", "Derived Data Resource Usage"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(ResourceUsageIcon);

	const FSlateIcon CacheStatisticsIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Statistics");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DerivedDataCacheStatisticsTabName, FOnSpawnTab::CreateRaw(this, &FDerivedDataEditor::CreateCacheStatisticsTab))
		.SetDisplayName(LOCTEXT("DerivedDataCacheStatisticsTabTitle", "Cache Statistics"))
		.SetTooltipText(LOCTEXT("DerivedDataCacheStatisticsTabToolTipText", "Derived Data Cache Statistics"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(CacheStatisticsIcon);

#if WITH_RELOAD
	// This code attempts to relaunch the tabs when you reload this module
	if (IsReloadActive() && FSlateApplication::IsInitialized())
	{
		ShowCacheStatisticsTab();
		ShowResourceUsageTab();
	}
#endif // WITH_RELOAD

	FDerivedDataEditorMenuCommands::Register();
}

void FDerivedDataEditor::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DerivedDataResourceUsageTabName);

		if (ResourceUsageTab.IsValid())
		{
			ResourceUsageTab.Pin()->RequestCloseTab();
		}

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DerivedDataCacheStatisticsTabName);

		if (CacheStatisticsTab.IsValid())
		{
			CacheStatisticsTab.Pin()->RequestCloseTab();
		}
	}

	FDerivedDataEditorMenuCommands::Unregister();
}

TSharedRef<SWidget> FDerivedDataEditor::CreateStatusBarWidget()
{
	return SNew(SDerivedDataStatusBarWidget);
}

TSharedPtr<SWidget> FDerivedDataEditor::CreateResourceUsageDialog()
{
	return SNew(SDerivedDataResourceUsageDialog);
}

TSharedRef<SDockTab> FDerivedDataEditor::CreateResourceUsageTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(ResourceUsageTab, SDockTab)
	.TabRole(ETabRole::NomadTab)
	[
		CreateResourceUsageDialog().ToSharedRef()
	];
}

void FDerivedDataEditor::ShowResourceUsageTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(DerivedDataResourceUsageTabName));
}


TSharedPtr<SWidget> FDerivedDataEditor::CreateCacheStatisticsDialog()
{
	return SNew(SDerivedDataCacheStatisticsDialog);
}


TSharedRef<SDockTab> FDerivedDataEditor::CreateCacheStatisticsTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(CacheStatisticsTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateCacheStatisticsDialog().ToSharedRef()
		];
}

void FDerivedDataEditor::ShowCacheStatisticsTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(DerivedDataCacheStatisticsTabName));
}

#undef LOCTEXT_NAMESPACE
