// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDerivedDataStatusBar.h"

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataEditor.h"
#include "DerivedDataInformation.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/UnitConversion.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "DerivedDataEditor"


TSharedRef<FUICommandList> FDerivedDataEditorMenuCommands::ActionList(new FUICommandList());

FDerivedDataEditorMenuCommands::FDerivedDataEditorMenuCommands()
	: TCommands<FDerivedDataEditorMenuCommands>
	(
		"DerivedDataSettings",
		NSLOCTEXT("Contexts", "Derived Data", "Derived Data"),
		"LevelEditor",
		FAppStyle::GetAppStyleSetName()
		)
{}

void FDerivedDataEditorMenuCommands::RegisterCommands()
{
	UI_COMMAND(ChangeSettings, "Change Cache Settings", "Opens a dialog to change Cache settings.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewCacheStatistics, "View Cache Statistics", "Opens the Cache Statistics panel.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewResourceUsage, "View Resource Usage", "Opens the Resource Usage panel.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(LaunchZenDashboard, "Launch Zen Dashboard", "Launches the Zen Dashboard utility.", EUserInterfaceActionType::Button, FInputChord());

	ActionList->MapAction(
		ChangeSettings,
		FExecuteAction::CreateStatic(&FDerivedDataEditorMenuCommands::ChangeSettings_Clicked)
	);

	ActionList->MapAction(
		ViewCacheStatistics,
		FExecuteAction::CreateStatic(&FDerivedDataEditorMenuCommands::ViewCacheStatistics_Clicked)
	);

	ActionList->MapAction(
		ViewResourceUsage,
		FExecuteAction::CreateStatic(&FDerivedDataEditorMenuCommands::ViewResourceUsage_Clicked)
	);
}

void FDerivedDataEditorMenuCommands::ChangeSettings_Clicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "Global");
}

void FDerivedDataEditorMenuCommands::ViewCacheStatistics_Clicked()
{
	FDerivedDataEditor& DerivedDataStatusBar = FModuleManager::LoadModuleChecked<FDerivedDataEditor>("DerivedDataEditor");
	DerivedDataStatusBar.ShowCacheStatisticsTab();
}

void FDerivedDataEditorMenuCommands::ViewResourceUsage_Clicked()
{
	FDerivedDataEditor& DerivedDataStatusBar = FModuleManager::LoadModuleChecked<FDerivedDataEditor>("DerivedDataEditor");
	DerivedDataStatusBar.ShowResourceUsageTab();
}

TSharedRef<SWidget> SDerivedDataStatusBarWidget::CreateStatusBarMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("StatusBar.ToolBar.DDC", NAME_None, EMultiBoxType::Menu, false);

	{
		FToolMenuSection& Section = Menu->AddSection("DDCMenuSettingsSection", LOCTEXT("DDCMenuSettingsSection", "Settings"));

		Section.AddMenuEntry(
			FDerivedDataEditorMenuCommands::Get().ChangeSettings,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Settings")
		);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("DDCMenuStatisticsSection", LOCTEXT("DDCMenuStatisticsSection", "Statistics"));

		Section.AddMenuEntry(
			FDerivedDataEditorMenuCommands::Get().ViewCacheStatistics,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.Cache.Statistics")
		);

		Section.AddMenuEntry(
			FDerivedDataEditorMenuCommands::Get().ViewResourceUsage,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DerivedData.ResourceUsage")
		);
	}

	return UToolMenus::Get()->GenerateWidget("StatusBar.ToolBar.DDC", FToolMenuContext(FDerivedDataEditorMenuCommands::ActionList));
}

void SDerivedDataStatusBarWidget::Construct(const FArguments& InArgs)
{	
	this->ChildSlot
	[
		SNew(SComboButton)
		.ContentPadding(FMargin(6.0f, 0.0f))
		.MenuPlacement(MenuPlacement_AboveAnchor)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.ButtonContent()
		[

			SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 3, 0)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image_Lambda([this] { return GetRemoteCacheStateBackgroundIcon();  })
					.ToolTipText_Lambda([this] { return GetRemoteCacheToolTipText(); })
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image_Lambda([this] { return GetRemoteCacheStateBadgeIcon();  })
					.ToolTipText_Lambda([this] { return GetRemoteCacheToolTipText(); })
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Uploading"))
					.ColorAndOpacity_Lambda([this] { return ( FDerivedDataInformation::IsUploading() && FDerivedDataInformation::GetRemoteCacheState() == ERemoteCacheState::Busy )? FLinearColor::White.CopyWithNewOpacity(FMath::MakePulsatingValue(ElapsedUploadTime, 2)) : FLinearColor(0,0,0,0); })
					.ToolTipText_Lambda([this] { return GetRemoteCacheToolTipText(); })
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Downloading"))
					.ColorAndOpacity_Lambda([this] { return ( FDerivedDataInformation::IsDownloading() && FDerivedDataInformation::GetRemoteCacheState()==ERemoteCacheState::Busy ) ? FLinearColor::White.CopyWithNewOpacity(FMath::MakePulsatingValue(ElapsedDownloadTime, 2)) : FLinearColor(0, 0, 0, 0); })
					.ToolTipText_Lambda([this] { return GetRemoteCacheToolTipText(); })
				]				
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 10, 0)
			[
				SNew(STextBlock)
				.Text_Lambda([this] { return GetTitleText(); })
				.ToolTipText_Lambda([this] { return GetTitleToolTipText(); })
			]
		]
		.OnGetMenuContent(FOnGetContent::CreateRaw(this, &SDerivedDataStatusBarWidget::CreateStatusBarMenu))
	];

	RegisterActiveTimer(0.2f, FWidgetActiveTimerDelegate::CreateSP(this, &SDerivedDataStatusBarWidget::UpdateBusyIndicator));
	RegisterActiveTimer(5.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDerivedDataStatusBarWidget::UpdateWarnings));
}

EActiveTimerReturnType SDerivedDataStatusBarWidget::UpdateBusyIndicator(double InCurrentTime, float InDeltaTime)
{	
	FDerivedDataInformation::UpdateRemoteCacheState();

	bBusy = GetDerivedDataCache()->AnyAsyncRequestsRemaining();

	if (FDerivedDataInformation::IsUploading())
	{
		ElapsedUploadTime += fmod(InDeltaTime,3600.0);
	}
	else
	{
		ElapsedUploadTime = 0.0;
	}

	if (FDerivedDataInformation::IsDownloading())
	{
		ElapsedDownloadTime += fmod(InDeltaTime, 3600.0);
	}
	else
	{
		ElapsedDownloadTime = 0.0;
	}

	if (bBusy)
	{
		ElapsedBusyTime += fmod(InDeltaTime, 3600.0);
	}
	else
	{
		ElapsedBusyTime = 0;
	}

	return EActiveTimerReturnType::Continue;
}

EActiveTimerReturnType SDerivedDataStatusBarWidget::UpdateWarnings(double InCurrentTime, float InDeltaTime)
{
	if ( FDerivedDataInformation::GetRemoteCacheState()== ERemoteCacheState::Warning )
	{
		if ( NotificationItem.IsValid() == false || NotificationItem->GetCompletionState()== SNotificationItem::CS_None)
		{
			// No existing notification or the existing one has finished
			TPromise<TWeakPtr<SNotificationItem>> NotificationPromise;

			FNotificationInfo Info(FDerivedDataInformation::GetRemoteCacheWarningMessage());
			Info.bUseSuccessFailIcons = true;
			Info.bFireAndForget = false;
			Info.bUseThrobber = false;
			Info.FadeOutDuration = 0.0f;
			Info.ExpireDuration = 0.0f;

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("OpenSettings", "Open Settings"), FText(), FSimpleDelegate::CreateLambda([NotificationFuture = NotificationPromise.GetFuture().Share()]() {
				FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "Global");

				TWeakPtr<SNotificationItem> NotificationPtr = NotificationFuture.Get();
				if (TSharedPtr<SNotificationItem> Notification = NotificationPtr.Pin())
				{
					Notification->SetCompletionState(SNotificationItem::CS_None);
					Notification->ExpireAndFadeout();
				}
			}),
				SNotificationItem::ECompletionState::CS_Fail));

			NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

			if (NotificationItem.IsValid())
			{
				NotificationPromise.SetValue(NotificationItem);
				NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
	}
	else
	{
		// No longer any warnings
		if (NotificationItem.IsValid())
		{
			NotificationItem->SetCompletionState(SNotificationItem::CS_None);
			NotificationItem->ExpireAndFadeout();
		}
	}

	return EActiveTimerReturnType::Continue;
}

FText SDerivedDataStatusBarWidget::GetTitleToolTipText() const
{
	FTextBuilder DescBuilder;

	DescBuilder.AppendLineFormat(LOCTEXT("GraphNameText", "Graph : {0}"), FText::FromString(GetDerivedDataCache()->GetGraphName()));
	
	return DescBuilder.ToText();
}

FText SDerivedDataStatusBarWidget::GetTitleText() const
{
	return LOCTEXT("DerivedDataToolBarName", "Derived Data");
}

FText SDerivedDataStatusBarWidget::GetRemoteCacheToolTipText() const
{
	FTextBuilder DescBuilder;

	if (FDerivedDataInformation::GetRemoteCacheState() == ERemoteCacheState::Warning)
	{
		DescBuilder.AppendLineFormat(LOCTEXT("RemoteCacheErrorText", "WARNING\t: {0}\n"), FDerivedDataInformation::GetRemoteCacheWarningMessage());
	}

	DescBuilder.AppendLine(LOCTEXT("RemoteCacheToolTipText", "Remote Cache\n"));
	DescBuilder.AppendLineFormat(LOCTEXT("RemoteCacheConnectedText", "Connected\t: {0}"), FText::FromString((FDerivedDataInformation::GetHasRemoteCache() ? TEXT("Yes") : TEXT("No"))));
	DescBuilder.AppendLineFormat(LOCTEXT("RemoteCacheStatusText", "Status\t: {0}"), FDerivedDataInformation::GetRemoteCacheStateAsText() );
	
	const double DownloadedBytesMB = FUnitConversion::Convert(FDerivedDataInformation::GetCacheActivitySizeBytes(true, false), EUnit::Bytes, EUnit::Megabytes);
	const double UploadedBytesMB = FUnitConversion::Convert(FDerivedDataInformation::GetCacheActivitySizeBytes(false, false), EUnit::Bytes, EUnit::Megabytes);

	DescBuilder.AppendLineFormat(LOCTEXT("RemoteCacheDownloaded", "Downloaded\t: {0} MB"), DownloadedBytesMB);
	DescBuilder.AppendLineFormat(LOCTEXT("RemoteCacheUploaded",	"Uploaded\t: {0} MB"), UploadedBytesMB);
	
	return DescBuilder.ToText();
}

const FSlateBrush* SDerivedDataStatusBarWidget::GetRemoteCacheStateBackgroundIcon() const
{
	switch ( FDerivedDataInformation::GetRemoteCacheState())
	{
		case ERemoteCacheState::Idle :
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.IdleBG");
		}

		case ERemoteCacheState::Busy :
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.BusyBG");
		}	

		case ERemoteCacheState::Unavailable:
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.UnavailableBG");
		}

		default:
		case ERemoteCacheState::Warning:
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.WarningBG");
		}
	}
}

const FSlateBrush* SDerivedDataStatusBarWidget::GetRemoteCacheStateBadgeIcon() const
{
	switch ( FDerivedDataInformation::GetRemoteCacheState())
	{
		case ERemoteCacheState::Idle :
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Idle");
		}

		case ERemoteCacheState::Busy :
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Busy");
		}	

		case ERemoteCacheState::Unavailable:
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Unavailable");
		}

		default:
		case ERemoteCacheState::Warning:
		{
			return FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Warning");
		}
	}
}

#undef LOCTEXT_NAMESPACE