// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandInfo;
class FUICommandList;
class SNotificationItem;
class SWidget;
struct FSlateBrush;

class FDerivedDataEditorMenuCommands : public TCommands<FDerivedDataEditorMenuCommands>
{
public:

	FDerivedDataEditorMenuCommands();

	virtual void RegisterCommands() override;

private:

	static void ChangeSettings_Clicked();
	static void ViewCacheStatistics_Clicked();
	static void ViewResourceUsage_Clicked();
	
public:

	TSharedPtr< FUICommandInfo > ChangeSettings;
	TSharedPtr< FUICommandInfo > ViewResourceUsage;
	TSharedPtr< FUICommandInfo > ViewCacheStatistics;
	TSharedPtr< FUICommandInfo > LaunchZenDashboard;

	static TSharedRef<FUICommandList> ActionList;
};


class SDerivedDataStatusBarWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDerivedDataStatusBarWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	FText						GetTitleToolTipText() const;
	FText						GetRemoteCacheToolTipText() const;
	FText						GetTitleText() const;
	const FSlateBrush*			GetRemoteCacheStateBackgroundIcon() const;
	const FSlateBrush*			GetRemoteCacheStateBadgeIcon() const;
	TSharedRef<SWidget>			CreateStatusBarMenu();
	EActiveTimerReturnType		UpdateBusyIndicator(double InCurrentTime, float InDeltaTime);
	EActiveTimerReturnType		UpdateWarnings(double InCurrentTime, float InDeltaTime);

	double ElapsedDownloadTime = 0;
	double ElapsedUploadTime = 0;
	double ElapsedBusyTime = 0;

	bool bBusy = false;

	TSharedPtr<SNotificationItem> NotificationItem;
};
